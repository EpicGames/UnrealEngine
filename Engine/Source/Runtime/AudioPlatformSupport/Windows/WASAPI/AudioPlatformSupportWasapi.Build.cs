// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AudioPlatformSupportWasapi : ModuleRules
	{
		public AudioPlatformSupportWasapi(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core"
				}
			);
		}
	}
}
