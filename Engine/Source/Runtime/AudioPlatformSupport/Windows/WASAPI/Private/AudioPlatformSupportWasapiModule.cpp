// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPlatformSupportWasapiLog.h"

#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogAudioPlatformSupportWasapi)


class FAudioPlatformSupportWasapiModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{

	}

	virtual void ShutdownModule() override
	{

	}
};

IMPLEMENT_MODULE(FAudioPlatformSupportWasapiModule, AudioPlatformSupportWasapi);
