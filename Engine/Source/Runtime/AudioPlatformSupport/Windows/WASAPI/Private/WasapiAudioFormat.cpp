// Copyright Epic Games, Inc. All Rights Reserved.

#include "WasapiAudioFormat.h"

#include "AudioPlatformSupportWasapiLog.h"

namespace Audio
{
	// For explanation on WAVEFORMATEX structure and cbSize requirements, see: 
	// https://learn.microsoft.com/en-us/windows/win32/api/mmeapi/ns-mmeapi-waveformatex
	static constexpr uint16 WaveFormatExtensibleCbsize = 22;
	static constexpr uint16 DefaultChannelCount = 2;
	static constexpr uint32 DefaultSampleRate = 44100;
	static constexpr EWasapiAudioEncoding DefaultEncoding = EWasapiAudioEncoding::PCM_16;

	FWasapiAudioFormat::FWasapiAudioFormat()
	{
		InitAudioFormat(DefaultChannelCount, DefaultSampleRate, DefaultEncoding);
		InitAudioEncoding();
	}

	FWasapiAudioFormat::FWasapiAudioFormat(WAVEFORMATEX* InFormat)
	{
		if ((InFormat->wFormatTag == WAVE_FORMAT_EXTENSIBLE) && (InFormat->cbSize >= WaveFormatExtensibleCbsize))
		{
			FMemory::Memcpy(&WaveFormat, InFormat, sizeof(WAVEFORMATEXTENSIBLE)); //-V512
		}
		else
		{
			FMemory::Memcpy(&WaveFormat, InFormat, sizeof(WAVEFORMATEX));
		}

		InitAudioEncoding();
	}

	FWasapiAudioFormat::FWasapiAudioFormat(uint16 InChannels, uint32 InSampleRate, EWasapiAudioEncoding InEncoding)
	{
		InitAudioFormat(InChannels, InSampleRate, InEncoding);
		InitAudioEncoding();
	}

	void FWasapiAudioFormat::InitAudioFormat(uint16 InChannels, uint32 InSampleRate, EWasapiAudioEncoding InEncoding)
	{
		static constexpr uint32 BitsPerByte = 8;
		uint16 BitDepth = EncodingToBitDepth(InEncoding);
		ensure(BitDepth != 0);

		if (BitDepth != 0)
		{
			WaveFormat.Format.wFormatTag = WAVE_FORMAT_EXTENSIBLE;
			WaveFormat.Format.cbSize = WaveFormatExtensibleCbsize;
			WaveFormat.Format.nChannels = InChannels;
			WaveFormat.Format.nSamplesPerSec = InSampleRate;
			WaveFormat.Format.wBitsPerSample = BitDepth;
			WaveFormat.Format.nBlockAlign = InChannels * (BitDepth / BitsPerByte);
			WaveFormat.Format.nAvgBytesPerSec = InSampleRate * WaveFormat.Format.nBlockAlign;
			WaveFormat.Samples.wValidBitsPerSample = BitDepth;

			if (InEncoding == EWasapiAudioEncoding::FLOATING_POINT_32 || InEncoding == EWasapiAudioEncoding::FLOATING_POINT_64)
			{
				WaveFormat.SubFormat = KSDATAFORMAT_SUBTYPE_IEEE_FLOAT;
			}
			else
			{
				WaveFormat.SubFormat = KSDATAFORMAT_SUBTYPE_PCM;
				if (InEncoding == EWasapiAudioEncoding::PCM_24_IN_32)
				{
					WaveFormat.Samples.wValidBitsPerSample = 24;
				}
			}
		}
	}

	void FWasapiAudioFormat::InitAudioEncoding()
	{
		Encoding = DetermineAudioEncoding(WaveFormat);
		if (Encoding == EWasapiAudioEncoding::UNKNOWN)
		{
			UE_LOGF(LogAudioPlatformSupportWasapi, Error, "FWasapiAudioFormat unknown audio format");
		}
	}

	EWasapiAudioEncoding FWasapiAudioFormat::GetEncoding() const
	{
		return Encoding;
	}

	uint32 FWasapiAudioFormat::GetNumChannels() const
	{
		return WaveFormat.Format.nChannels;
	}

	uint32 FWasapiAudioFormat::GetSampleRate() const
	{
		return WaveFormat.Format.nSamplesPerSec;
	}

	uint32 FWasapiAudioFormat::GetBitsPerSample() const
	{
		return WaveFormat.Format.wBitsPerSample;
	}

	uint32 FWasapiAudioFormat::GetBytesPerSample() const
	{
		return WaveFormat.Format.wBitsPerSample / 8;
	}

	uint32 FWasapiAudioFormat::GetFrameSizeInBytes() const
	{
		return WaveFormat.Format.nBlockAlign;
	}

	const WAVEFORMATEX* FWasapiAudioFormat::GetWaveFormat() const
	{
		return &(WaveFormat.Format);
	}

	EWasapiAudioEncoding FWasapiAudioFormat::DetermineAudioEncoding(const WAVEFORMATEXTENSIBLE& InFormat)
	{
		if (InFormat.Format.wFormatTag == WAVE_FORMAT_IEEE_FLOAT)
		{
			if (InFormat.Format.wBitsPerSample == (sizeof(float) * 8))
			{
				return EWasapiAudioEncoding::FLOATING_POINT_32;
			}
			else if (InFormat.Format.wBitsPerSample == (sizeof(double) * 8))
			{
				return EWasapiAudioEncoding::FLOATING_POINT_64;
			}
			else
			{
				return EWasapiAudioEncoding::UNKNOWN;
			}
		}
		else
		{
			if (InFormat.Format.wFormatTag == WAVE_FORMAT_PCM)
			{
				switch (InFormat.Format.wBitsPerSample)
				{
				case 8:
					return EWasapiAudioEncoding::PCM_8;

				case 16:
					return EWasapiAudioEncoding::PCM_16;

				case 24:
					return EWasapiAudioEncoding::PCM_24;

				case 32:
					return EWasapiAudioEncoding::PCM_32;

				default:
					return EWasapiAudioEncoding::UNKNOWN;
				}
			}
			else
			{
				if ((InFormat.Format.wFormatTag == WAVE_FORMAT_EXTENSIBLE) && (InFormat.Format.cbSize >= WaveFormatExtensibleCbsize))
				{
					if (InFormat.SubFormat == KSDATAFORMAT_SUBTYPE_IEEE_FLOAT)
					{
						if (InFormat.Format.wBitsPerSample == (sizeof(float) * 8))
						{
							return EWasapiAudioEncoding::FLOATING_POINT_32;
						}
						else if (InFormat.Format.wBitsPerSample == (sizeof(double) * 8))
						{
							return EWasapiAudioEncoding::FLOATING_POINT_64;
						}
						else
						{
							return EWasapiAudioEncoding::UNKNOWN;
						}
					}
					else
					{
						if (InFormat.SubFormat == KSDATAFORMAT_SUBTYPE_PCM)
						{
							switch (InFormat.Samples.wValidBitsPerSample)
							{
							case 8:
								return (InFormat.Format.wBitsPerSample == 8) ? EWasapiAudioEncoding::PCM_8 : EWasapiAudioEncoding::UNKNOWN;

							case 16:
								return (InFormat.Format.wBitsPerSample == 16) ? EWasapiAudioEncoding::PCM_16 : EWasapiAudioEncoding::UNKNOWN;

							case 32:
								return (InFormat.Format.wBitsPerSample == 32) ? EWasapiAudioEncoding::PCM_32 : EWasapiAudioEncoding::UNKNOWN;

							case 24:
								if (InFormat.Format.wBitsPerSample == 32)
								{
									return EWasapiAudioEncoding::PCM_24_IN_32;
								}
								else
								{
									if (InFormat.Format.wBitsPerSample == 24)
									{
										return EWasapiAudioEncoding::PCM_24;
									}
									else
									{
										return EWasapiAudioEncoding::UNKNOWN;
									}
								}
							default:
								break;
							}
						}
					}
				}
			}
		}

		return EWasapiAudioEncoding::UNKNOWN;
	}

	uint16 FWasapiAudioFormat::EncodingToBitDepth(EWasapiAudioEncoding InEncoding)
	{
		switch (InEncoding)
		{
		case EWasapiAudioEncoding::PCM_8:
			return 8;

		case EWasapiAudioEncoding::PCM_16:
			return 16;

		case EWasapiAudioEncoding::PCM_24:
			return 24;

		case EWasapiAudioEncoding::PCM_24_IN_32:
		case EWasapiAudioEncoding::PCM_32:
		case EWasapiAudioEncoding::FLOATING_POINT_32:
			return 32;

		case EWasapiAudioEncoding::FLOATING_POINT_64:
			return 64;

		default:
			return 0;
		}
	}

	FString FWasapiAudioFormat::GetEncodingString() const
	{
		switch (GetEncoding())
		{
		case EWasapiAudioEncoding::PCM_8:
			return TEXT("8-Bit PCM");

		case EWasapiAudioEncoding::PCM_16:
			return TEXT("16-Bit PCM");

		case EWasapiAudioEncoding::PCM_24:
			return TEXT("24-Bit PCM");

		case EWasapiAudioEncoding::PCM_24_IN_32:
			return TEXT("24-Bit PCM (32-bit Container)");

		case EWasapiAudioEncoding::PCM_32:
			return TEXT("PCM_32 PCM");

		case EWasapiAudioEncoding::FLOATING_POINT_32:
			return TEXT("32-Bit Floating Point");

		case EWasapiAudioEncoding::FLOATING_POINT_64:
			return TEXT("64-Bit Floating Point");

		default:
			return TEXT("UNKNOWN");
		}
	}

}
