// Copyright Epic Games, Inc. All Rights Reserved.

#include "WasapiAudioUtils.h"

#include "AudioPlatformSupportWasapiLog.h"

namespace Audio
{
	// REFERENCE_TIME base which is in units of 100 nanoseconds
	static constexpr REFERENCE_TIME ReferenceTimeBase = 1e7;

	REFERENCE_TIME FWasapiAudioUtils::FramesToRefTime(const uint32 InNumFrames, const uint32 InSampleRate)
	{
		REFERENCE_TIME RefTimeDuration = FMath::RoundToInt(static_cast<double>(InNumFrames * ReferenceTimeBase) / InSampleRate);

		return RefTimeDuration;
	}

	uint64 FWasapiAudioUtils::RefTimeToFrames(const REFERENCE_TIME InRefTime, const uint32 InSampleRate)
	{
		uint64 NumFrames = FMath::RoundToInt(static_cast<double>(InRefTime * InSampleRate) / ReferenceTimeBase);

		return NumFrames;
	}
}
