// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

THIRD_PARTY_INCLUDES_START
#include <AudioClient.h>
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"

#define UE_API AUDIOPLATFORMSUPPORTWASAPI_API


namespace Audio
{
	/**
	 * EWasapiAudioEncoding - Audio bit depths supported by WASAPI implementation
	 */
	enum class EWasapiAudioEncoding : uint8
	{
		UNKNOWN,
		PCM_8,
		PCM_16,
		PCM_24,
		PCM_24_IN_32,
		PCM_32,
		FLOATING_POINT_32,
		FLOATING_POINT_64
	};

	/**
	 * FWasapiAudioFormat - Wrapper class for Windows WAVEFORMATEX and WAVEFORMATEXTENSIBLE structs.
	 */
	class FWasapiAudioFormat
	{
	public:
		/** FWasapiAudioFormat - Constructs underlying WAVEFORMATEX with default values. */
		UE_API FWasapiAudioFormat();

		/**
		 * FWasapiAudioFormat - Constructs underlying data structure with given WAVEFORMATEX parameter.
		 * 
		 * @param InFormat - Pointer to a WAVEFORMATEX struct from which to construct this object. 
		 * Correctly handles the extended form, WAVEFORMATEXTENSIBLE. 
		 */
		UE_API explicit FWasapiAudioFormat(WAVEFORMATEX* InFormat);

		/**
		 * FWasapiAudioFormat - Constructs underlying data structure using given parameters.
		 * 
		 * @param InChannels - Number of audio channels.
		 * @param InSampleRate - Samples per second (Hz).
		 * @param InEncoding - For PCM data, this is the bit depth of data (e.g. 16-bit integer, 32-bit float, etc.)
		 */
		UE_API FWasapiAudioFormat(uint16 InChannels, uint32 InSampleRate, EWasapiAudioEncoding InEncoding);

		/** GetEncoding - Returns the encoding for this audio format. */
		UE_API EWasapiAudioEncoding GetEncoding() const;

		/** GetNumChannels - Returns the number of audio channels present in this format. */
		UE_API uint32 GetNumChannels() const;

		/** GetSampleRate - Returns the audio sample rate for this format. */
		UE_API uint32 GetSampleRate() const;

		/** GetBitsPerSample - Returns the bit depth for the audio data as an integer value. */
		UE_API uint32 GetBitsPerSample() const;

		/** GetBytesPerSample - Returns the number of bytes needed to store a single sample of audio data. */
		UE_API uint32 GetBytesPerSample() const;

		/** GetFrameSizeInBytes - Returns the number of bytes needed to store a singe frame of audio data. */
		UE_API uint32 GetFrameSizeInBytes() const;
		
		/** GetEncodingString - Returns string form of encoding. Suitable for logging. */
		UE_API FString GetEncodingString() const;

		/**
		 * GetWaveFormat - Returns pointer to raw WAVEFORMATEX struct. Use this for specific OS API calls
		 * which require it. Otherwise use the accessors above.
		 */
		UE_API const WAVEFORMATEX* GetWaveFormat() const;

	private:

		/** InitAudioFormat - Init method used by constructors to initialize data. */
		void InitAudioFormat(uint16 InChannels, uint32 InSampleRate, EWasapiAudioEncoding InEncoding);
		/** InitAudioEncoding - Parses underlying WAVEFORMATEXTENSIBLE structure and determines corresponding encoding. */
		void InitAudioEncoding();
		/** DetermineAudioEncoding - Helper function for determining audio encoding from WAVEFORMATEXTENSIBLE struct. */
		static EWasapiAudioEncoding DetermineAudioEncoding(const WAVEFORMATEXTENSIBLE& InFormat);
		/** EncodingToBitDepth - Helper function for getting bit depth from InEncoding parameter. */
		static uint16 EncodingToBitDepth(EWasapiAudioEncoding InEncoding);

		/** WaveFormat - Windows wave format struct which this object wraps. */
		WAVEFORMATEXTENSIBLE WaveFormat = {};
		/** Encoding - Audio encoding for this object. */
		EWasapiAudioEncoding Encoding = EWasapiAudioEncoding::UNKNOWN;
	};
}

#undef UE_API
