// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

THIRD_PARTY_INCLUDES_START
#include <AudioClient.h>
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"

#define UE_API AUDIOPLATFORMSUPPORTWASAPI_API


namespace Audio
{
	/**
	 * FWasapiAudioUtils - WASAPI utility class
	 */
	class FWasapiAudioUtils
	{
	public:
		/** FramesToRefTime - Converts a given number of frames at the given sample rate to REFERENCE_TIME. */
		static UE_API REFERENCE_TIME FramesToRefTime(const uint32 InNumFrames, const uint32 InSampleRate);

		/** RefTimeToFrames - Converts the given REFERENCE_TIME to a number of frames at the given sample rate. */
		static UE_API uint64 RefTimeToFrames(const REFERENCE_TIME InRefTime, const uint32 InSampleRate);
	};
}

#undef UE_API
